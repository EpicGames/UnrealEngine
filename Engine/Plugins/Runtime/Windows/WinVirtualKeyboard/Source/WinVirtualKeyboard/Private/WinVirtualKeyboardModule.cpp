// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/IPlatformTextField.h"
#include "Framework/Application/SlateApplication.h"

#if WITH_DEBUG_DRAW
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#endif

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#include <sdkddkver.h>
#include <roapi.h>
#include <wrl.h>
#include <windows.ui.viewmanagement.core.h>
#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

using namespace ABI::Windows;
using namespace ABI::Windows::UI::ViewManagement::Core;
using namespace Microsoft::WRL;
using namespace Microsoft::WRL::Wrappers;


/* 
 * The ICoreInputView::OcclusionsChanged event takes some time to come through after the Virtual Keyboard has been shown
 * and the ICoreInputView::GetCoreInputViewOcclusions often produces no data at all.
 * 
 * Deferring OnVirtualKeyboardShown gives time for the OcclusionsChanged to arrive. Note that No events will be sent if
 * the keyboard is hidden before the delay finishes.
 * 
 * If you don't need the occlusion rect, this can be set to 0 and WinVirtualKeyboard.WithOcclusionRect can be set to false.
 */
static float GShowEventDelaySeconds = 0.3f;
static FAutoConsoleVariableRef CVarShowEventDelaySeconds(
	TEXT("WinVirtualKeyboard.ShowEventDelay"),
	GShowEventDelaySeconds,
	TEXT("Delay in seconds before sending 'show' event")
);

static bool GWithOcclusionRect = true;
static FAutoConsoleVariableRef CVarWithOcclusionRect(
	TEXT("WinVirtualKeyboard.WithOcclusionRect"),
	GWithOcclusionRect,
	TEXT("Whether to send the occluded area with the 'show' event")
);

#if WITH_DEBUG_DRAW
static bool GDrawOcclusionRect = false;
static FAutoConsoleVariableRef CVarDrawOcclusionRect(
	TEXT("WinVirtualKeyboard.DrawOcclusionRect"),
	GDrawOcclusionRect,
	TEXT("Whether to draw the last occluded area sent with the 'show' event")
);
#endif


#if defined(NTDDI_VERSION) && defined(NTDDI_WIN11_GE) && (NTDDI_VERSION > NTDDI_WIN11_GE) // definition was introduced in 10.0.26100.3624 so once we're past 26100, assume it is available
	static const CoreInputViewKind DesiredInputKind = CoreInputViewKind_Gamepad;
#else
	static const CoreInputViewKind DesiredInputKind = (CoreInputViewKind)7;
#endif


class FWindowsPlatformTextField : public IPlatformTextField
{
public:
	FWindowsPlatformTextField( ComPtr<ICoreInputView> InView, ComPtr<ICoreInputView3> InView3, ComPtr<ICoreInputView4> InView4 )
		: View(InView)
		, View3(InView3)
		, View4(InView4)
	{
	}

	virtual ~FWindowsPlatformTextField()
	{
		UnregisterEvents();
	}


	virtual void ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) override
	{
		HRESULT hResult;
		boolean Success = 0;

		if (bShow)
		{
			hResult = View3->TryShowWithKind(DesiredInputKind, &Success);
		}
		else
		{
			hResult = View3->TryHide(&Success);
		}

		bool bSuccess = SUCCEEDED(hResult) && Success;
		UE_CLOGF(!bSuccess, PLATFORM_GLOBAL_LOG_CATEGORY, Error, "WinVirtualKeyboard %ls failed. hResult=0x%X, Result=%ls", bShow ? TEXT("show") : TEXT("hide"), hResult, Success ? TEXT("true") : TEXT("false"));
	}


	HRESULT OnVirtualKeyboardShowing(ICoreInputView*, ICoreInputViewShowingEventArgs*)
	{
		if (!DeferredShowDelegate.IsValid())
		{
			// delay sending the OnVirtualKeyboardShown to give time for OnOcclusionsChanged
			auto DeferredShowCallback = [this]( float DeltaTime )
			{
				FPlatformRect VkRect = GWithOcclusionRect ? LastOcclusionRect : FPlatformRect{0,0,0,0};
				FSlateApplication::Get().GetPlatformApplication()->OnVirtualKeyboardShown().Broadcast(VkRect);
				SetDebugDrawRect(VkRect);

				DeferredShowDelegate.Reset();
				return false;
			};
			DeferredShowDelegate = FTSTicker::GetCoreTicker().AddTicker( FTickerDelegate::CreateLambda(DeferredShowCallback), GShowEventDelaySeconds );
		}

		return S_OK;
	}


	HRESULT OnVirtualKeyboardHiding(ICoreInputView*, ICoreInputViewHidingEventArgs*)
	{
		if (DeferredShowDelegate.IsValid())
		{
			// virtual keyboard was hidden again before the timer expired - don't send any events
			FTSTicker::GetCoreTicker().RemoveTicker(DeferredShowDelegate);
			DeferredShowDelegate.Reset();
		}
		else
		{
			FSlateApplication::Get().GetPlatformApplication()->OnVirtualKeyboardHidden().Broadcast();
			ClearDebugDrawRect();
		}

		return S_OK;
	}


	HRESULT OnOcclusionsChanged( ICoreInputView*, ICoreInputViewOcclusionsChangedEventArgs* Args )
	{
		HRESULT hResult;

		ComPtr<Foundation::Collections::IVectorView<CoreInputViewOcclusion*>> Occlusions;
		hResult = Args->get_Occlusions(&Occlusions);
		if (SUCCEEDED(hResult))
		{
			unsigned int OcclusionsNum = 0;
			hResult = Occlusions->get_Size(&OcclusionsNum);
			if (SUCCEEDED(hResult) && OcclusionsNum > 0)
			{
				// compute the bounding box for all occulsions
				bool bFound = false;
				FPlatformRect MaxRect(INT32_MAX,INT32_MAX,INT32_MIN,INT32_MIN);
				for (unsigned int OcclusionIndex = 0; OcclusionIndex < OcclusionsNum; OcclusionIndex++)
				{
					ComPtr<ICoreInputViewOcclusion> Occlusion;
					hResult = Occlusions->GetAt(OcclusionIndex, &Occlusion);
					if (SUCCEEDED(hResult))
					{
						Foundation::Rect OccludingRect;
						hResult = Occlusion->get_OccludingRect(&OccludingRect);
						if (SUCCEEDED(hResult) && OccludingRect.Y != 0) // documentation says Y == 0 means the app window is not obstructed by the pane
						{
							MaxRect.Left   = FMath::Min(MaxRect.Left,   FMath::RoundToInt32(OccludingRect.X));
							MaxRect.Top    = FMath::Min(MaxRect.Top,    FMath::RoundToInt32(OccludingRect.Y));
							MaxRect.Right  = FMath::Max(MaxRect.Right,  FMath::RoundToInt32(OccludingRect.X) + FMath::RoundToInt32(OccludingRect.Width));
							MaxRect.Bottom = FMath::Max(MaxRect.Bottom, FMath::RoundToInt32(OccludingRect.Y) + FMath::RoundToInt32(OccludingRect.Height));

							bFound = true;
						}
					}
				}

				if (bFound)
				{
					LastOcclusionRect = MaxRect;
				}
			}
		}

		return S_OK;
	}


	bool RegisterEvents()
	{
		HRESULT hResult;

		// register callback when virtual keyboard is shown
		hResult = View4->add_PrimaryViewShowing(Callback<Foundation::ITypedEventHandler<CoreInputView*, CoreInputViewShowingEventArgs*>>(this, &FWindowsPlatformTextField::OnVirtualKeyboardShowing).Get(), &PrimaryViewShowingToken);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Error, "WinVirtualKeyboard will be unavailable: can't register 'showing' callback. hResult=0x%X", hResult);
			return false;
		}

		// register callback when virtual keyboard is hidden
		hResult = View4->add_PrimaryViewHiding(Callback<Foundation::ITypedEventHandler<CoreInputView*, CoreInputViewHidingEventArgs*>>(this, &FWindowsPlatformTextField::OnVirtualKeyboardHiding).Get(), &PrimaryViewHidingToken);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Error, "WinVirtualKeyboard will be unavailable: can't register 'hiding' callback. hResult=0x%X", hResult);
			return false;
		}

		// register a callback when the occlusions change. this can be when the virtual keyboard is shown, hidden or moved by the user
		hResult = View->add_OcclusionsChanged(Callback<Foundation::ITypedEventHandler<CoreInputView*, CoreInputViewOcclusionsChangedEventArgs*>>(this, &FWindowsPlatformTextField::OnOcclusionsChanged).Get(), &OcclusionsChangedToken);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Error, "WinVirtualKeyboard will be unavailable: can't register 'occlusion changed' callback. hResult=0x%X", hResult);
			return false;
		}

		return true;
	}


	void UnregisterEvents()
	{
		if (DeferredShowDelegate.IsValid())
		{
			FTSTicker::GetCoreTicker().RemoveTicker(DeferredShowDelegate);
			DeferredShowDelegate.Reset();
		}

		View4->remove_PrimaryViewShowing(PrimaryViewShowingToken);
		View4->remove_PrimaryViewHiding(PrimaryViewHidingToken);
		View->remove_OcclusionsChanged(OcclusionsChangedToken);

		ClearDebugDrawRect();
	}

	void SetDebugDrawRect( const FPlatformRect& DrawRect)
	{
#if WITH_DEBUG_DRAW
		ClearDebugDrawRect();

		if (GDrawOcclusionRect)
		{
			auto DebugDraw = [this, DebugRect = DrawRect]( UCanvas* Canvas, APlayerController*)
			{
				FCanvasBoxItem Box( FVector2D(DebugRect.Left, DebugRect.Top), FVector2D(DebugRect.Right-DebugRect.Left, DebugRect.Bottom-DebugRect.Top) );
				Box.SetColor(FLinearColor::Yellow);
				Box.LineThickness = 4.0f;
				Canvas->DrawItem(Box);
			};
			DebugDrawDelegate = UDebugDrawService::Register( TEXT("Game"), FDebugDrawDelegate::CreateLambda(DebugDraw));
		}
#endif
	}


	void ClearDebugDrawRect()
	{
#if WITH_DEBUG_DRAW
		UDebugDrawService::Unregister(DebugDrawDelegate);
		DebugDrawDelegate.Reset();
#endif
	}

private:
	FPlatformRect LastOcclusionRect{0,0,0,0};

	ComPtr<ICoreInputView> View;
	ComPtr<ICoreInputView3> View3;
	ComPtr<ICoreInputView4> View4;

	EventRegistrationToken PrimaryViewShowingToken{0};
	EventRegistrationToken PrimaryViewHidingToken{0};
	EventRegistrationToken OcclusionsChangedToken{0};

	FTSTicker::FDelegateHandle DeferredShowDelegate;

#if WITH_DEBUG_DRAW
	FDelegateHandle DebugDrawDelegate;
#endif
};


class FWindowsVirtualKeyboardModule : public IModuleInterface, public IPlatformTextFieldFactory
{
public:

	// IModuleInterface
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(IPlatformTextFieldFactory::FeatureName, this);
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(IPlatformTextFieldFactory::FeatureName, this);
	}

	
	// IPlatformTextFieldFactory
	virtual TUniquePtr<IPlatformTextField> CreateInstance() override
	{
		HRESULT hResult;

		// get the view
		// @note: RoGetActivationFactory requires Win8 or higher... should this be preceeded with LoadLibrary(combase.dll) / GetProcAddress() ?
		ComPtr<ICoreInputViewStatics> ViewStatics;
		hResult = RoGetActivationFactory( HStringReference(RuntimeClass_Windows_UI_ViewManagement_Core_CoreInputView).Get(),  __uuidof(ICoreInputViewStatics), &ViewStatics);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Warning, "WinVirtualKeyboard will be unavailable: can't get view statics 0x%X", hResult);
			return nullptr;
		}

		ComPtr<ICoreInputView> View;
		hResult = ViewStatics->GetForCurrentView(&View);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Warning, "WinVirtualKeyboard will be unavailable: can't get current view 0x%X", hResult);
			return nullptr;
		}

		ComPtr<ICoreInputView3> View3;
		hResult = View.As(&View3);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Warning, "WinVirtualKeyboard will be unavailable: can't get view3 0x%X", hResult);
			return nullptr;
		}

		ComPtr<ICoreInputView4> View4;
		hResult = View.As(&View4);
		if (FAILED(hResult))
		{
			UE_LOGF(PLATFORM_GLOBAL_LOG_CATEGORY, Warning, "WinVirtualKeyboard will be unavailable: can't get view4 0x%X", hResult);
			return nullptr;
		}

		TUniquePtr<FWindowsPlatformTextField> Instance = MakeUnique<FWindowsPlatformTextField>(View, View3, View4);
		if (!Instance->RegisterEvents())
		{
			return nullptr;
		}

		return MoveTemp(Instance);
	}
};


IMPLEMENT_MODULE(FWindowsVirtualKeyboardModule, WinVirtualKeyboard);


