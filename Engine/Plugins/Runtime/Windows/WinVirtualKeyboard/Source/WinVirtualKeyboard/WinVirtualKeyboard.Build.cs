// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WinVirtualKeyboard : ModuleRules
	{
		// adds the WinVirtualKeyboard.DrawOcclusionRect console command to visualize the occlusion rect in non-shipping builds
		static readonly bool bWithDebugRendering = true;

		public WinVirtualKeyboard(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.Add("SlateCore");
			PrivateDependencyModuleNames.Add("Slate");
			PrivateDependencyModuleNames.Add("ApplicationCore");
			PublicDependencyModuleNames.Add("Core");
			PublicSystemLibraries.Add("WindowsApp.lib");

			if (bWithDebugRendering && Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDefinitions.Add("WITH_DEBUG_DRAW=1");
				PublicDependencyModuleNames.Add("Engine");
			}
			else
			{
				PrivateDefinitions.Add("WITH_DEBUG_DRAW=0");
			}
		}
	}
}
