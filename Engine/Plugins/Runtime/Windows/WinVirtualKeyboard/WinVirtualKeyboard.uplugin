{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Windows Virtual Keyboard",
	"Description": "Virtual Keyboard support for Windows. Requires Windows 11 26100.5061 or later",
	"Category": "Input Devices",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"RequiresBuildPlatform": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "WinVirtualKeyboard",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ],
			"TargetDenyList": [ "Server" ]
		}
	]
}